{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Link KiPro Device",
	"Description": "AJA Ki Pro recorder support for Live Link Hub with recording and connection capabilities",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveLinkKiProDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "LiveLinkDevice",
			"Enabled": true
		},
		{
			"Name": "LiveLinkGenericRecordingDevice",
			"Enabled": true
		},
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		}
	]
}
