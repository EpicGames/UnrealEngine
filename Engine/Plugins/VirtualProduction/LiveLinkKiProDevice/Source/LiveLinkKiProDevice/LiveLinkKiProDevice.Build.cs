// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkKiProDevice : ModuleRules
{
	public LiveLinkKiProDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"LiveLinkDevice",
				"LiveLinkInterface",
				"Projects",
				"HTTP",
				"Json",
				"JsonUtilities",
				"Networking"
			}
		);
	}
}
