// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkKiProDeviceBase.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Runtime/Networking/Public/Interfaces/IPv4/IPv4Endpoint.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#include "Internationalization/Regex.h"

#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "TimerManager.h"
#include "Engine/World.h"
#include "Engine/Engine.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "GenericPlatform/GenericPlatformHttp.h"

#define LOCTEXT_NAMESPACE "LiveLinkKiProDevice"

//~ Begin ULiveLinkDevice interface

TSubclassOf<ULiveLinkDeviceSettings> ULiveLinkKiProDeviceBase::GetSettingsClass() const
{
	return SettingsClass ? SettingsClass : TSubclassOf<ULiveLinkDeviceSettings>(ULiveLinkKiProDeviceSettings::StaticClass());
}

EDeviceHealth ULiveLinkKiProDeviceBase::GetDeviceHealth() const
{
	switch (ConnectionStatus)
	{
	case ELiveLinkDeviceConnectionStatus::Connected:
		return bIsRecording ? EDeviceHealth::Good : EDeviceHealth::Nominal;
	case ELiveLinkDeviceConnectionStatus::Connecting:
		return EDeviceHealth::Info;
	case ELiveLinkDeviceConnectionStatus::Disconnecting:
		return EDeviceHealth::Warning;
	case ELiveLinkDeviceConnectionStatus::Disconnected:
	default:
		return EDeviceHealth::Error;
	}
}

FText ULiveLinkKiProDeviceBase::GetHealthText() const
{
	switch (ConnectionStatus)
	{
	case ELiveLinkDeviceConnectionStatus::Connected:
		return bIsRecording ? LOCTEXT("Recording", "Recording") : LOCTEXT("Connected", "Connected - Ready");
	case ELiveLinkDeviceConnectionStatus::Connecting:
		return LOCTEXT("Connecting", "Connecting...");
	case ELiveLinkDeviceConnectionStatus::Disconnecting:
		return LOCTEXT("Disconnecting", "Disconnecting...");
	case ELiveLinkDeviceConnectionStatus::Disconnected:
	default:
		return LOCTEXT("Disconnected", "Disconnected");
	}
}

void ULiveLinkKiProDeviceBase::OnDeviceAdded()
{
	// Subscribe to recording session delegates for slate/take changes
	ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
	Session.OnSlateNameChanged().AddUObject(this, &ULiveLinkKiProDeviceBase::HandleSlateNameChanged);
	Session.OnTakeNumberChanged().AddUObject(this, &ULiveLinkKiProDeviceBase::HandleTakeNumberChanged);

	// Start main device ticker (handles reconnection and polling)
	// Use high frequency (0.1s) for transport polling, accumulate time for reconnection
	DeviceTickerHandle =
		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateUObject(this, &ULiveLinkKiProDeviceBase::DeviceTick),
			PollingIntervalSeconds
		);

	ULiveLinkKiProDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkKiProDeviceSettings>();
	DeviceSettings->DisplayName = FString::Printf(TEXT("Ki Pro (%s)"), *DeviceSettings->IpAddress);
}

void ULiveLinkKiProDeviceBase::OnDeviceRemoved()
{
	// Unsubscribe from recording session delegates
	ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
	Session.OnSlateNameChanged().RemoveAll(this);
	Session.OnTakeNumberChanged().RemoveAll(this);

	// Stop device ticker
	if (DeviceTickerHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(DeviceTickerHandle);
		DeviceTickerHandle.Reset();
	}

	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		Disconnect_Implementation();
	}
}


void ULiveLinkKiProDeviceBase::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(ULiveLinkDeviceSettings, DisplayName))
	{
		return;
	}

	// When settings change, attempt immediate reconnection if disconnected
	// This provides instant feedback when user updates IP address or port
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		UE_LOGF(LogTemp, Log, "Settings changed, attempting connection to Ki Pro...");
		AttemptReconnection();
	}
}

ELiveLinkDeviceConnectionStatus ULiveLinkKiProDeviceBase::GetConnectionStatus_Implementation() const
{
	return ConnectionStatus;
}

FString ULiveLinkKiProDeviceBase::GetHardwareId_Implementation() const
{
	const ULiveLinkKiProDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkKiProDeviceSettings>();
	return FString::Printf(TEXT("%s:%d"), *DeviceSettings->IpAddress, DeviceSettings->Port);
}

bool ULiveLinkKiProDeviceBase::SetHardwareId_Implementation(const FString& HardwareID)
{
	ULiveLinkKiProDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkKiProDeviceSettings>();

	FIPv4Endpoint OutEndpoint;
	if (FIPv4Endpoint::Parse(HardwareID, OutEndpoint))
	{
		DeviceSettings->IpAddress = OutEndpoint.Address.ToString();
		DeviceSettings->Port = OutEndpoint.Port;
		return true;
	}

	return false;
}

bool ULiveLinkKiProDeviceBase::CanSetHardwareId_Implementation()
{
	// Can set hardware ID when disconnected
	return ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected;
}

bool ULiveLinkKiProDeviceBase::Connect_Implementation()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		return false;
	}

	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Connecting;

	// Test connection by getting firmware version
	GetParameter(TEXT("eParamID_SWVersion"), [this](bool bSuccess, const FString& Value, const FString& Text)
	{
		if (bSuccess)
		{
			ULiveLinkKiProDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkKiProDeviceSettings>();

			// Text contains the encoded version as a string (e.g., "67239937")
			// Convert to int and decode to human-readable format (e.g., "4.2.0.1")
			int32 EncodedVersion = FCString::Atoi(*Text);
			DeviceSettings->FirmwareVersion = DecodeFirmwareVersion(EncodedVersion);

			UE_LOGF(LogTemp, Log, "Connected to Ki Pro at %ls, Firmware: %ls", *DeviceSettings->IpAddress, *DeviceSettings->FirmwareVersion);

			// Query the device for its transport command values before marking fully connected
			QueryTransportCommands([this](bool bCommandsSuccess)
			{
				if (bCommandsSuccess)
				{
					ConnectionStatus = ELiveLinkDeviceConnectionStatus::Connected;
				}
				else
				{
					UE_LOGF(LogTemp, Error, "Failed to query transport commands from Ki Pro");
					ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;
				}
			});
		}
		else
		{
			ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;
			UE_LOGF(LogTemp, Error, "Failed to connect to Ki Pro - device unresponsive");
		}
	});

	return true;
}

bool ULiveLinkKiProDeviceBase::Disconnect_Implementation()
{
	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnecting;

	// Stop any polling (handled by main ticker now)
	CurrentPollingOperation = EPollingOperation::None;
	bIsRecording = false;
	TransportCommandValues.Empty();
	ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;
	UE_LOGF(LogTemp, Log, "Disconnected from Ki Pro");

	return true;
}

bool ULiveLinkKiProDeviceBase::StartRecording_Implementation()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Connected)
	{
		UE_LOGF(LogTemp, Warning, "Cannot start recording - not connected");
		return false;
	}

	// Get slate/take from recording session
	ILiveLinkRecordingSession& RecordingSession = ILiveLinkRecordingSession::Get();

	const FString Slate = RecordingSession.GetSlateName();
	const int32 Take = RecordingSession.GetTakeNumber();

	UE_LOGF(LogTemp, Log, "Starting recording: Slate=%ls, Take=%d", *Slate, Take);

	// Set media state for recording
	SetMediaStateForRecordPlay();

	// Set slate/take metadata
	SetParameter(TEXT("eParamID_CustomClipName"), Slate, [](bool bSuccess) {});
	SetParameter(TEXT("eParamID_CustomTake"), FString::FromInt(Take), [](bool bSuccess) {});

	// Send record command
	SendTransportCommand(TEXT("Record Command"));

	// Begin polling for state change (handled by main ticker)
	CurrentPollingOperation = EPollingOperation::WaitingForRecording;
	PollingStartTime = FPlatformTime::Seconds();

	return true;
}

bool ULiveLinkKiProDeviceBase::StopRecording_Implementation()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Connected)
	{
		UE_LOGF(LogTemp, Warning, "Cannot stop recording - not connected");
		return false;
	}

	UE_LOGF(LogTemp, Log, "Stopping recording");

	// Set media state for recording
	SetMediaStateForRecordPlay();

	// Send stop command
	SendTransportCommand(TEXT("Stop Command"));

	// Begin polling for state change to Idle (handled by main ticker)
	CurrentPollingOperation = EPollingOperation::WaitingForIdle;
	PollingStartTime = FPlatformTime::Seconds();

	return true;
}

bool ULiveLinkKiProDeviceBase::IsRecording_Implementation() const
{
	return bIsRecording;
}

//~ End ILiveLinkDeviceCapability_Recording interface

//~ Begin Private Methods

FString ULiveLinkKiProDeviceBase::GetBaseUrl() const
{
	const ULiveLinkKiProDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkKiProDeviceSettings>();
	return FString::Printf(TEXT("http://%s:%d"), *DeviceSettings->IpAddress, DeviceSettings->Port);
}

FString ULiveLinkKiProDeviceBase::DecodeFirmwareVersion(int32 VersionBits)
{
	// Decode 32-bit integer to dotted version string
	// Format: (major<<24) | (minor<<16) | (patch<<8) | build
	// Example: 67239937 (0x04020001) -> "4.2.0.1"
	const uint8 Major = (VersionBits & 0xFF000000) >> 24;
	const uint8 Minor = (VersionBits & 0x00FF0000) >> 16;
	const uint8 Patch = (VersionBits & 0x0000FF00) >> 8;
	const uint8 Build = (VersionBits & 0x000000FF);

	return FString::Printf(TEXT("%d.%d.%d.%d"), Major, Minor, Patch, Build);
}

void ULiveLinkKiProDeviceBase::GetParameter(const FString& ParamId, TFunction<void(bool bSuccess, const FString& Value, const FString& Text)> OnComplete)
{
	// KiPro devices are usually on the local network and 2.0s should be a reasonable default timeout.
	const float HttpRequestTimeout = 2.0f;
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->SetURL(GetBaseUrl() + TEXT("/options?") + ParamId);
	HttpRequest->SetTimeout(HttpRequestTimeout);

	// Capture weak pointer to protect against device destruction before callback
	TWeakObjectPtr<ULiveLinkKiProDeviceBase> WeakThis(this);
	HttpRequest->OnProcessRequestComplete().BindWeakLambda(this, [WeakThis, OnComplete](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful)
	{
		// Check if device still exists before executing callback
		if (WeakThis.IsValid())
		{
			HandleGetParameterResponse(Request, Response, bWasSuccessful, OnComplete);
		}
	});

	HttpRequest->ProcessRequest();
}

void ULiveLinkKiProDeviceBase::SetParameter(const FString& ParamId, const FString& Value, TFunction<void(bool bSuccess)> OnComplete)
{
	// KiPro devices are usually on the local network and 2.0s should be a reasonable default timeout.
	const float HttpRequestTimeout = 2.0f;
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetVerb(TEXT("GET"));

	FString EncodedValue = FGenericPlatformHttp::UrlEncode(Value);
	FString URL = FString::Printf(TEXT("%s/config?action=set&paramid=%s&value=%s"), *GetBaseUrl(), *ParamId, *EncodedValue);
	HttpRequest->SetURL(URL);
	HttpRequest->SetTimeout(HttpRequestTimeout);

	// Capture weak pointer to protect against device destruction before callback
	TWeakObjectPtr<ULiveLinkKiProDeviceBase> WeakThis(this);
	HttpRequest->OnProcessRequestComplete().BindWeakLambda(this, [WeakThis, OnComplete](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful)
	{
		// Check if device still exists before executing callback
		if (WeakThis.IsValid())
		{
			HandleSetParameterResponse(Request, Response, bWasSuccessful, OnComplete);
		}
	});

	HttpRequest->ProcessRequest();
}

void ULiveLinkKiProDeviceBase::GetTransportState(TFunction<void(bool bSuccess, const FString& StateText)> OnComplete)
{
	GetParameter(TEXT("eParamID_TransportState"), [OnComplete](bool bSuccess, const FString& Value, const FString& Text)
	{
		OnComplete(bSuccess, Text);
	});
}

void ULiveLinkKiProDeviceBase::SendTransportCommand(const FString& CommandText)
{
	const int32* FoundValue = TransportCommandValues.Find(CommandText);
	if (!FoundValue)
	{
		UE_LOGF(LogTemp, Error, "Unknown transport command: %ls (device reported %d available commands)", *CommandText, TransportCommandValues.Num());
		return;
	}

	SetParameter(TEXT("eParamID_TransportCommand"), FString::FromInt(*FoundValue), [CommandText](bool bSuccess)
	{
		if (!bSuccess)
		{
			UE_LOGF(LogTemp, Error, "Failed to send transport command: %ls", *CommandText);
		}
	});
}

void ULiveLinkKiProDeviceBase::QueryTransportCommands(TFunction<void(bool bSuccess)> OnComplete)
{
	const float HttpRequestTimeout = 5.0f;
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->SetURL(GetBaseUrl() + TEXT("/options?eParamID_TransportCommand"));
	HttpRequest->SetTimeout(HttpRequestTimeout);

	TWeakObjectPtr<ULiveLinkKiProDeviceBase> WeakThis(this);
	HttpRequest->OnProcessRequestComplete().BindWeakLambda(this, [WeakThis, OnComplete](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful)
	{
		if (!WeakThis.IsValid())
		{
			return;
		}

		ULiveLinkKiProDeviceBase* Self = WeakThis.Get();

		if (!bWasSuccessful || !Response.IsValid() || Response->GetResponseCode() != 200)
		{
			OnComplete(false);
			return;
		}

		FString ResponseContent = Response->GetContentAsString();

		// Sanitize Ki Pro JavaScript notation to valid JSON
		ResponseContent.TrimEndInline();
		if (ResponseContent.EndsWith(TEXT(";")))
		{
			ResponseContent.LeftChopInline(1);
			ResponseContent.TrimEndInline();
		}

		FRegexPattern PropertyPattern(TEXT("([,{]\\s*)([a-zA-Z_][a-zA-Z0-9_]*)\\s*:"));
		FRegexMatcher PropertyMatcher(PropertyPattern, ResponseContent);
		FString SanitizedContent;
		int32 LastEndPos = 0;
		while (PropertyMatcher.FindNext())
		{
			SanitizedContent.Append(ResponseContent.Mid(LastEndPos, PropertyMatcher.GetMatchBeginning() - LastEndPos));
			SanitizedContent.Append(PropertyMatcher.GetCaptureGroup(1));
			SanitizedContent.Append(TEXT("\""));
			SanitizedContent.Append(PropertyMatcher.GetCaptureGroup(2));
			SanitizedContent.Append(TEXT("\":"));
			LastEndPos = PropertyMatcher.GetMatchEnding();
		}
		SanitizedContent.Append(ResponseContent.Mid(LastEndPos));

		TArray<TSharedPtr<FJsonValue>> OptionsArray;
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(SanitizedContent);

		if (!FJsonSerializer::Deserialize(JsonReader, OptionsArray))
		{
			UE_LOGF(LogTemp, Error, "Failed to parse transport command options from Ki Pro");
			OnComplete(false);
			return;
		}

		Self->TransportCommandValues.Empty();
		for (const TSharedPtr<FJsonValue>& OptionValue : OptionsArray)
		{
			TSharedPtr<FJsonObject> OptionObject = OptionValue->AsObject();
			if (OptionObject.IsValid())
			{
				FString CommandText, CommandValueStr;
				OptionObject->TryGetStringField(TEXT("text"), CommandText);
				OptionObject->TryGetStringField(TEXT("value"), CommandValueStr);

				if (!CommandText.IsEmpty())
				{
					Self->TransportCommandValues.Add(CommandText, FCString::Atoi(*CommandValueStr));
				}
			}
		}

		UE_LOGF(LogTemp, Log, "Queried %d transport commands from Ki Pro", Self->TransportCommandValues.Num());
		OnComplete(Self->TransportCommandValues.Num() > 0);
	});

	HttpRequest->ProcessRequest();
}

void ULiveLinkKiProDeviceBase::SetMediaStateForRecordPlay()
{
	// Check if in Data-LAN mode and switch to Record-Play (value 0)
	GetParameter(TEXT("eParamID_MediaState"), [this](bool bSuccess, const FString& Value, const FString& Text)
	{
		if (bSuccess && Text == TEXT("Data - LAN"))
		{
			UE_LOGF(LogTemp, Log, "Switching Ki Pro from Data-LAN to Record-Play mode");
			SetParameter(TEXT("eParamID_MediaState"), TEXT("0"), [](bool bSuccess) {});
		}
	});
}

void ULiveLinkKiProDeviceBase::PollTransportState()
{
	// Prevent request flooding on slow connections - don't send new request if one is in-flight
	if (bPollingRequestInFlight)
	{
		return;
	}

	// Check for timeout
	if (FPlatformTime::Seconds() - PollingStartTime > PollingTimeoutSeconds)
	{
		UE_LOGF(LogTemp, Warning, "Timeout waiting for transport state change");
		CurrentPollingOperation = EPollingOperation::None;
		bPollingRequestInFlight = false;
		return;
	}

	// Mark request as in-flight before sending
	bPollingRequestInFlight = true;

	// Poll transport state
	GetTransportState([this](bool bSuccess, const FString& StateText)
	{
		OnTransportStateReceived(bSuccess, StateText);
	});
}

void ULiveLinkKiProDeviceBase::OnTransportStateReceived(bool bSuccess, const FString& StateText)
{
	// Clear in-flight flag now that request has completed
	bPollingRequestInFlight = false;

	if (!bSuccess)
	{
		UE_LOGF(LogTemp, Error, "Failed to get transport state");
		CurrentPollingOperation = EPollingOperation::None;
		return;
	}

	bool bOperationComplete = false;

	switch (CurrentPollingOperation)
	{
	case EPollingOperation::WaitingForRecording:
		if (StateText == TEXT("Recording"))
		{
			bIsRecording = true;
			bOperationComplete = true;
			UE_LOGF(LogTemp, Log, "Recording started on Ki Pro");
		}
		break;

	case EPollingOperation::WaitingForIdle:
		if (StateText == TEXT("Idle"))
		{
			bIsRecording = false;
			bOperationComplete = true;
			OnRecordingStopped();
		}
		break;

	case EPollingOperation::WaitingForPlay:
		if (StateText == TEXT("Playing Forward"))
		{
			bOperationComplete = true;
			UE_LOGF(LogTemp, Log, "Playback started on Ki Pro");
		}
		break;

	default:
		break;
	}

	if (bOperationComplete)
	{
		CurrentPollingOperation = EPollingOperation::None;
	}
}

void ULiveLinkKiProDeviceBase::OnRecordingStopped()
{
	// Get clip name for logging
	GetParameter(TEXT("eParamID_CurrentClip"), [this](bool bSuccess, const FString& Value, const FString& Text)
	{
		if (bSuccess)
		{
			UE_LOGF(LogTemp, Log, "Recording stopped. Clip: %ls", *Text);
		}

		// Auto-play if enabled
		const ULiveLinkKiProDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkKiProDeviceSettings>();
		if (DeviceSettings->bAutoPlayAfterStop)
		{
			UE_LOGF(LogTemp, Log, "Auto-play enabled, starting playback");
			SendTransportCommand(TEXT("Play Command"));

			// Continue polling for playback state (handled by main ticker)
			CurrentPollingOperation = EPollingOperation::WaitingForPlay;
			PollingStartTime = FPlatformTime::Seconds();
		}
	});
}

void ULiveLinkKiProDeviceBase::HandleGetParameterResponse(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful, TFunction<void(bool, const FString&, const FString&)> OnComplete)
{
	if (!bWasSuccessful || !Response.IsValid() || Response->GetResponseCode() != 200)
	{
		OnComplete(false, FString(), FString());
		return;
	}

	FString ResponseContent = Response->GetContentAsString();

	// Sanitize response: AJA Ki Pro returns JavaScript notation, not valid JSON
	// - Remove trailing semicolons: [...}]; -> [...}]
	// - Add quotes around unquoted property names: {value: -> {"value":
	ResponseContent.TrimEndInline();
	if (ResponseContent.EndsWith(TEXT(";")))
	{
		ResponseContent.LeftChopInline(1);
		ResponseContent.TrimEndInline();
	}

	// Fix unquoted property names using regex pattern
	// Matches: word characters followed by colon (not already quoted)
	// Replace: add quotes around the property name
	FRegexPattern PropertyPattern(TEXT("([,{]\\s*)([a-zA-Z_][a-zA-Z0-9_]*)\\s*:"));
	FRegexMatcher PropertyMatcher(PropertyPattern, ResponseContent);
	FString SanitizedContent;
	int32 LastEndPos = 0;
	while (PropertyMatcher.FindNext())
	{
		// Append text before match
		SanitizedContent.Append(ResponseContent.Mid(LastEndPos, PropertyMatcher.GetMatchBeginning() - LastEndPos));

		// Append sanitized property: prefix + "propertyName":
		SanitizedContent.Append(PropertyMatcher.GetCaptureGroup(1)); // comma/brace and whitespace
		SanitizedContent.Append(TEXT("\""));
		SanitizedContent.Append(PropertyMatcher.GetCaptureGroup(2)); // property name
		SanitizedContent.Append(TEXT("\":"));

		LastEndPos = PropertyMatcher.GetMatchEnding();
	}
	// Append remaining text
	SanitizedContent.Append(ResponseContent.Mid(LastEndPos));

	// Parse JSON response - AJA returns array directly, not wrapped in object
	TArray<TSharedPtr<FJsonValue>> OptionsArray;
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(SanitizedContent);

	if (FJsonSerializer::Deserialize(JsonReader, OptionsArray))
	{
		// Find selected option (AJA uses "true"/"false" strings, not booleans)
		for (const TSharedPtr<FJsonValue>& OptionValue : OptionsArray)
		{
			TSharedPtr<FJsonObject> OptionObject = OptionValue->AsObject();
			if (OptionObject.IsValid())
			{
				FString SelectedStr;
				OptionObject->TryGetStringField(TEXT("selected"), SelectedStr);

				if (SelectedStr == TEXT("true"))
				{
					FString Value, Text;
					OptionObject->TryGetStringField(TEXT("value"), Value);
					OptionObject->TryGetStringField(TEXT("text"), Text);
					OnComplete(true, Value, Text);
					return;
				}
			}
		}
	}

	OnComplete(false, FString(), FString());
}

void ULiveLinkKiProDeviceBase::HandleSetParameterResponse(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful, TFunction<void(bool)> OnComplete)
{
	OnComplete(bWasSuccessful && Response.IsValid() && Response->GetResponseCode() == 200);
}

bool ULiveLinkKiProDeviceBase::DeviceTick(float DeltaTime)
{
	// Handle periodic reconnection attempts
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		ReconnectionTimeAccumulator += DeltaTime;
		if (ReconnectionTimeAccumulator >= ReconnectionIntervalSeconds)
		{
			ReconnectionTimeAccumulator = 0.0f;
			AttemptReconnection();
		}
	}
	else
	{
		// Reset accumulator when connected
		ReconnectionTimeAccumulator = 0.0f;
	}

	// Handle transport state polling
	if (CurrentPollingOperation != EPollingOperation::None)
	{
		PollTransportState();
	}

	// Return true to continue ticking
	return true;
}

void ULiveLinkKiProDeviceBase::AttemptReconnection()
{
	UE_LOGF(LogTemp, Log, "Attempting to reconnect to Ki Pro...");
	ILiveLinkDeviceCapability_Connection::Execute_Connect(this);
}

void ULiveLinkKiProDeviceBase::HandleSlateNameChanged(FStringView InSlateName)
{
	// Update slate name on device if connected
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected)
	{
		FString SlateName(InSlateName);
		UE_LOGF(LogTemp, Log, "Slate name changed to: %ls", *SlateName);
		SetParameter(TEXT("eParamID_CustomClipName"), SlateName, [](bool bSuccess) {});
	}
}

void ULiveLinkKiProDeviceBase::HandleTakeNumberChanged(int32 InTakeNumber)
{
	// Update take number on device if connected
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected)
	{
		UE_LOGF(LogTemp, Log, "Take number changed to: %d", InTakeNumber);
		SetParameter(TEXT("eParamID_CustomTake"), FString::FromInt(InTakeNumber), [](bool bSuccess) {});
	}
}

//~ End Private Methods

#undef LOCTEXT_NAMESPACE
