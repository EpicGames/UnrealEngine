// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "LiveLinkKiProDeviceSettings.h"
#include "Interfaces/IHttpRequest.h"
#include "Containers/Ticker.h"

#include "LiveLinkKiProDeviceBase.generated.h"

/**
 * Concrete implementation for AJA Ki Pro recording devices
 * Provides both Connection and Recording capabilities via REST API
 */
UCLASS(BlueprintType, meta = (DisplayName = "KiPro Device", ToolTip = "Support for AJA KiPro"))
class LIVELINKKIPRODEVICE_API ULiveLinkKiProDeviceBase : public ULiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
	, public ILiveLinkDeviceCapability_Recording
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override;
	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;
	virtual void OnDeviceAdded() override;
	virtual void OnDeviceRemoved() override;
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) override;
	virtual bool CanSetHardwareId_Implementation() override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	//~ Begin ILiveLinkDeviceCapability_Recording interface
	virtual bool StartRecording_Implementation() override;
	virtual bool StopRecording_Implementation() override;
	virtual bool IsRecording_Implementation() const override;
	//~ End ILiveLinkDeviceCapability_Recording interface

private:
	/** Build URL for Ki Pro REST API */
	FString GetBaseUrl() const;

	/** Decode firmware version from 32-bit integer to dotted string (e.g., 67239937 -> "4.2.0.1") */
	static FString DecodeFirmwareVersion(int32 VersionBits);

	/** Get a parameter from the Ki Pro device */
	void GetParameter(const FString& ParamId, TFunction<void(bool bSuccess, const FString& Value, const FString& Text)> OnComplete);

	/** Set a parameter on the Ki Pro device */
	void SetParameter(const FString& ParamId, const FString& Value, TFunction<void(bool bSuccess)> OnComplete);

	/** Get the transport state */
	void GetTransportState(TFunction<void(bool bSuccess, const FString& StateText)> OnComplete);

	/** Send a transport command using the cached value for the given command text */
	void SendTransportCommand(const FString& CommandText);

	/** Query the device for available transport commands and cache their values */
	void QueryTransportCommands(TFunction<void(bool bSuccess)> OnComplete);

	/** Ensure device is in Record-Play mode (not Data-LAN) */
	void SetMediaStateForRecordPlay();

	/** Poll transport state until target state is reached */
	void PollTransportState();

	/** Called when transport state polling completes */
	void OnTransportStateReceived(bool bSuccess, const FString& StateText);

	/** Called when recording stop is confirmed */
	void OnRecordingStopped();

	/** Handle HTTP request completion for GetParameter */
	static void HandleGetParameterResponse(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful, TFunction<void(bool, const FString&, const FString&)> OnComplete);

	/** Handle HTTP request completion for SetParameter */
	static void HandleSetParameterResponse(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful, TFunction<void(bool)> OnComplete);

	/** Main device tick function (handles reconnection and polling) */
	bool DeviceTick(float DeltaTime);

	/** Attempt reconnection if disconnected */
	void AttemptReconnection();

	/** Handle slate name changes from recording session */
	void HandleSlateNameChanged(FStringView InSlateName);

	/** Handle take number changes from recording session */
	void HandleTakeNumberChanged(int32 InTakeNumber);

protected:
	/** Settings class to use for Ki Pro devices */
	UPROPERTY(EditDefaultsOnly, Category="Live Link Device")
	TSubclassOf<ULiveLinkDeviceSettings> SettingsClass;

	/** Current connection status */
	UPROPERTY()
	ELiveLinkDeviceConnectionStatus ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;

	/** Whether the device is currently recording */
	UPROPERTY()
	bool bIsRecording = false;

	/** Ticker handle for main device tick (reconnection and polling) */
	FTSTicker::FDelegateHandle DeviceTickerHandle;

	/** What operation we're polling for */
	enum class EPollingOperation
	{
		None,
		WaitingForRecording,
		WaitingForIdle,
		WaitingForPlay
	};
	EPollingOperation CurrentPollingOperation = EPollingOperation::None;

	/** Time when polling started (for timeout) */
	double PollingStartTime = 0.0;

	/** Time accumulator for reconnection attempts */
	float ReconnectionTimeAccumulator = 0.0f;

	/** Flag to prevent polling request flood on slow connections */
	bool bPollingRequestInFlight = false;

	/** Cached transport command text -> numeric value mapping, queried from the device */
	TMap<FString, int32> TransportCommandValues;

	/** Polling timeout in seconds */
	static constexpr double PollingTimeoutSeconds = 4.0;

	/** Polling interval in seconds */
	static constexpr float PollingIntervalSeconds = 0.1f;

	/** Reconnection attempt interval in seconds */
	static constexpr float ReconnectionIntervalSeconds = 5.0f;
};
