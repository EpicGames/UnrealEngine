// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkDevice.h"
#include "LiveLinkKiProDeviceSettings.generated.h"

/**
 * Settings for AJA Ki Pro recording device
 */
UCLASS()
class LIVELINKKIPRODEVICE_API ULiveLinkKiProDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:
	/** IP address of the Ki Pro device */
	UPROPERTY(EditAnywhere, Category="Ki Pro", meta=(DisplayName="IP Address"))
	FString IpAddress = TEXT("192.168.1.100");

	/** HTTP port for REST API communication */
	UPROPERTY(EditAnywhere, Category="Ki Pro", AdvancedDisplay)
	uint16 Port = 80;

	/** Automatically start playback after recording stops */
	UPROPERTY(EditAnywhere, Category="Ki Pro", meta=(DisplayName="Auto Play After Stop Recording"))
	bool bAutoPlayAfterStop = false;

	/** Firmware version retrieved from device (read-only) */
	UPROPERTY(VisibleAnywhere, Category="Ki Pro", AdvancedDisplay, meta=(DisplayName="Firmware Version"))
	FString FirmwareVersion;
};
