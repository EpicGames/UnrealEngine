# Mock Ki Pro Device Simulator

A Python-based simulator that implements the AJA Ki Pro REST API for testing the Live Link Ki Pro plugin without physical hardware.

## Features

- Implements key Ki Pro REST API endpoints
- Simulates transport state machine (Idle → Recording → Idle)
- Tracks recording parameters (slate, take, clip name)
- Media state management (Record-Play vs Data-LAN mode)
- Real-time command logging
- Configurable IP and port

## Requirements

- Python 3.6 or higher
- No external dependencies (uses Python standard library)

## Usage

### Basic Usage (Port 80)

Run as administrator/root since port 80 requires elevated privileges:

```bash
# Windows (run as Administrator)
python mock_kipro_device.py

# Linux/Mac (run with sudo)
sudo python3 mock_kipro_device.py
```

### Custom Port (No Admin Required)

```bash
python mock_kipro_device.py --port 8080
```

### Custom IP Address

```bash
python mock_kipro_device.py --ip 192.168.1.100 --port 80
```

### All Options

```bash
python mock_kipro_device.py --help
```

## Testing with Live Link Hub

1. **Start the simulator:**
   ```bash
   python mock_kipro_device.py --port 8080
   ```

2. **Launch Live Link Hub**

3. **Add Ki Pro Device:**
   - Click "Add Device"
   - Select "Ki Pro Device"
   - Device appears in device list

4. **Configure IP Address:**
   - Edit device settings
   - Set IP Address: `127.0.0.1` (for local testing)
   - Set Port: `8080` (or 80 if using default)

5. **Connect:**
   - Click "Connect" button in device row
   - Simulator will log: `[GET] eParamID_SWVersion -> 4.2.0.1`
   - Device status should show "Connected - Ready"

6. **Test Recording:**
   - Set slate and take in Live Link Hub recording session
   - Click "Start Recording"
   - Simulator logs:
     ```
     [SET] eParamID_CustomClipName = MySlate
     [SET] eParamID_CustomTake = 1
     [TRANSPORT COMMAND] Record Command (value=4)
       -> Recording started: MySlate_Take1
     ```
   - Click "Stop Recording"
   - Simulator logs:
     ```
     [TRANSPORT COMMAND] Stop Command (value=1)
       -> Stopped (was Recording)
     ```

## Simulator Output Example

```
============================================================
Mock Ki Pro Device Simulator
============================================================
Listening on: http://0.0.0.0:8080
Firmware Version: 4.2.0.1

Waiting for connections from Live Link Hub...
Press Ctrl+C to stop
============================================================

[GET] eParamID_SWVersion -> 4.2.0.1
[MEDIA STATE] Changed to: Record - Play
[SLATE] Set to: Scene_1_Shot_A
[TAKE] Set to: 3

[TRANSPORT COMMAND] Record Command (value=4)
  -> Recording started: Scene_1_Shot_A_Take3

[GET] eParamID_TransportState -> Recording

[TRANSPORT COMMAND] Stop Command (value=1)
  -> Stopped (was Recording)

[GET] eParamID_CurrentClip -> Scene_1_Shot_A_Take3
```

## Implemented API Endpoints

### GET /options?<param_id>

Returns parameter information with current value and options.

**Example:** `GET /options?eParamID_SWVersion`

**Response:**
```json
{
  "param_id": "eParamID_SWVersion",
  "options": [
    {
      "value": "4020001",
      "text": "4.2.0.1",
      "selected": true
    }
  ]
}
```

### GET /config?action=set&paramid=<id>&value=<val>

Sets a parameter value.

**Example:** `GET /config?action=set&paramid=eParamID_CustomClipName&value=MySlate`

**Response:**
```json
{
  "status": "ok"
}
```

## Supported Parameters

- `eParamID_SWVersion` - Firmware version (read-only)
- `eParamID_TransportState` - Current transport state (Idle, Recording, Playing Forward)
- `eParamID_MediaState` - Media state (Record-Play, Data-LAN)
- `eParamID_TransportCommand` - Transport commands (Stop=1, Play=3, Record=4)
- `eParamID_CustomClipName` - Slate name
- `eParamID_CustomTake` - Take number
- `eParamID_CurrentClip` - Current clip name (generated from slate+take)

## Transport State Machine

```
Idle → Record Command → Recording
Recording → Stop Command → Idle
Idle → Play Command → Playing Forward
Playing Forward → Stop Command → Idle
```

## Troubleshooting

### Port 80 Permission Denied

On Linux/Mac, port 80 requires root:
```bash
sudo python3 mock_kipro_device.py
```

Or use a higher port:
```bash
python3 mock_kipro_device.py --port 8080
```

### Connection Refused

- Check firewall settings
- Verify IP address and port match in Live Link Hub settings
- Ensure simulator is running before attempting connection

### Device Not Appearing in Live Link Hub

- Rebuild the plugin after C++ changes
- Verify the plugin is enabled in Live Link Hub
- Check that `ULiveLinkDeviceKiProBase` is not marked as `Abstract`

## Known Limitations

- Does not implement file download/Data-LAN mode functionality
- Simplified transport command mapping (hard-coded values)
- No authentication
- Single-threaded (one request at a time)
- No persistent storage of clips

## Development Notes

The simulator tracks state in memory and resets on restart. It's designed for basic functional testing of the Live Link Ki Pro plugin's connection, recording, and transport control features.

For more advanced testing scenarios, you can modify the Python script to add:
- Delayed state transitions (simulate real hardware timing)
- Error injection (simulate connection failures)
- Additional parameters
- File system integration
