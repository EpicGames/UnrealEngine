#!/usr/bin/env python3
"""
Mock AJA Ki Pro Device Simulator

Simulates the REST API of an AJA Ki Pro device for testing the Live Link Ki Pro plugin.
Implements the key endpoints and parameter management needed for recording control.

Usage:
    python mock_kipro_device.py [--port PORT] [--ip IP]

Example:
    python mock_kipro_device.py --port 80 --ip 192.168.1.100
"""

import argparse
import json
import time
from http.server import HTTPServer, BaseHTTPRequestHandler
from urllib.parse import urlparse, parse_qs
from threading import Lock


class KiProState:
    """Manages the state of the mock Ki Pro device"""

    def __init__(self):
        self.lock = Lock()

        # Device parameters
        self.firmware_version = "4.2.0.1"
        self.transport_state = "Idle"  # Idle, Recording, Playing Forward, etc.
        self.media_state = "Record - Play"  # "Record - Play" or "Data - LAN"
        self.custom_clip_name = "DefaultClip"
        self.custom_take = "1"
        self.current_clip = ""

        # Transport state machine
        self.state_transitions = {
            "Idle": ["Recording", "Playing Forward"],
            "Recording": ["Idle"],
            "Playing Forward": ["Idle"],
        }

        # Parameter definitions (param_id -> {value, text, options})
        # NOTE: AJA uses "true"/"false" STRINGS for selected, not booleans
        # NOTE: For version params, "text" must be encoded as: (major<<24)|(minor<<16)|(patch<<8)|build
        #       Version 4.2.0.1 = (4<<24)|(2<<16)|(0<<8)|1 = 67239937 = 0x04020001
        self.parameters = {
            "eParamID_SWVersion": {
                "value": "67239937",
                "text": "67239937",  # Encoded: 4.2.0.1 -> 0x04020001 -> 67239937
                "options": [{"value": "67239937", "text": "67239937", "selected": "true"}]
            },
            "eParamID_TransportState": {
                "value": "0",
                "text": "Idle",
                "options": [
                    {"value": "0", "text": "Idle", "selected": "true"},
                    {"value": "1", "text": "Recording", "selected": "false"},
                    {"value": "2", "text": "Playing Forward", "selected": "false"},
                ]
            },
            "eParamID_MediaState": {
                "value": "0",
                "text": "Record - Play",
                "options": [
                    {"value": "0", "text": "Record - Play", "selected": "true"},
                    {"value": "1", "text": "Data - LAN", "selected": "false"},
                ]
            },
            "eParamID_TransportCommand": {
                "value": "0",
                "text": "No Command",
                "options": [
                    {"value": "0", "text": "No Command", "selected": "true"},
                    {"value": "1", "text": "Stop Command", "selected": "false"},
                    {"value": "2", "text": "Play Reverse Command", "selected": "false"},
                    {"value": "3", "text": "Play Command", "selected": "false"},
                    {"value": "4", "text": "Record Command", "selected": "false"},
                ]
            },
            "eParamID_CustomClipName": {
                "value": "DefaultClip",
                "text": "DefaultClip",
                "options": [{"value": "DefaultClip", "text": "DefaultClip", "selected": "true"}]
            },
            "eParamID_CustomTake": {
                "value": "1",
                "text": "1",
                "options": [{"value": "1", "text": "1", "selected": "true"}]
            },
            "eParamID_CurrentClip": {
                "value": "",
                "text": "",
                "options": [{"value": "", "text": "", "selected": "true"}]
            },
        }

    def get_parameter(self, param_id):
        """Get parameter value and options"""
        with self.lock:
            if param_id in self.parameters:
                return self.parameters[param_id]
            return None

    def set_parameter(self, param_id, value):
        """Set parameter value"""
        with self.lock:
            if param_id == "eParamID_TransportCommand":
                return self._handle_transport_command(value)
            elif param_id == "eParamID_MediaState":
                return self._set_media_state(value)
            elif param_id == "eParamID_CustomClipName":
                return self._set_custom_clip_name(value)
            elif param_id == "eParamID_CustomTake":
                return self._set_custom_take(value)
            else:
                print(f"Warning: Unhandled set parameter: {param_id} = {value}")
                return True

    def _handle_transport_command(self, value):
        """Handle transport command"""
        command_map = {
            "1": "Stop Command",
            "3": "Play Command",
            "4": "Record Command",
        }

        command = command_map.get(value, "Unknown")
        print(f"\n[TRANSPORT COMMAND] {command} (value={value})")

        if value == "4":  # Record
            if self.transport_state == "Idle":
                self.transport_state = "Recording"
                self._update_transport_state_param()
                # Generate clip name from slate and take
                clip_name = f"{self.custom_clip_name}_Take{self.custom_take}"
                self.current_clip = clip_name
                self._update_current_clip_param(clip_name)
                print(f"  -> Recording started: {clip_name}")
            else:
                print(f"  -> Cannot record from state: {self.transport_state}")

        elif value == "1":  # Stop
            if self.transport_state in ["Recording", "Playing Forward"]:
                prev_state = self.transport_state
                self.transport_state = "Idle"
                self._update_transport_state_param()
                print(f"  -> Stopped (was {prev_state})")
            else:
                print(f"  -> Already in state: {self.transport_state}")

        elif value == "3":  # Play
            if self.transport_state == "Idle":
                self.transport_state = "Playing Forward"
                self._update_transport_state_param()
                print(f"  -> Playback started")
            else:
                print(f"  -> Cannot play from state: {self.transport_state}")

        return True

    def _set_media_state(self, value):
        """Set media state (Record-Play vs Data-LAN)"""
        if value == "0":
            self.media_state = "Record - Play"
            text = "Record - Play"
        elif value == "1":
            self.media_state = "Data - LAN"
            text = "Data - LAN"
        else:
            return False

        print(f"[MEDIA STATE] Changed to: {text}")

        # Update parameter (use string "true"/"false")
        param = self.parameters["eParamID_MediaState"]
        param["value"] = value
        param["text"] = text
        for opt in param["options"]:
            opt["selected"] = "true" if opt["value"] == value else "false"

        return True

    def _set_custom_clip_name(self, value):
        """Set custom clip name (slate)"""
        self.custom_clip_name = value
        print(f"[SLATE] Set to: {value}")

        # Update parameter
        param = self.parameters["eParamID_CustomClipName"]
        param["value"] = value
        param["text"] = value
        param["options"] = [{"value": value, "text": value, "selected": "true"}]

        return True

    def _set_custom_take(self, value):
        """Set custom take number"""
        self.custom_take = value
        print(f"[TAKE] Set to: {value}")

        # Update parameter
        param = self.parameters["eParamID_CustomTake"]
        param["value"] = value
        param["text"] = value
        param["options"] = [{"value": value, "text": value, "selected": "true"}]

        return True

    def _update_transport_state_param(self):
        """Update transport state parameter"""
        state_map = {
            "Idle": "0",
            "Recording": "1",
            "Playing Forward": "2",
        }

        value = state_map.get(self.transport_state, "0")
        param = self.parameters["eParamID_TransportState"]
        param["value"] = value
        param["text"] = self.transport_state

        # Update selected option (use string "true"/"false")
        for opt in param["options"]:
            opt["selected"] = "true" if opt["text"] == self.transport_state else "false"

    def _update_current_clip_param(self, clip_name):
        """Update current clip parameter"""
        param = self.parameters["eParamID_CurrentClip"]
        param["value"] = clip_name
        param["text"] = clip_name
        param["options"] = [{"value": clip_name, "text": clip_name, "selected": "true"}]


class KiProRequestHandler(BaseHTTPRequestHandler):
    """HTTP request handler for Ki Pro REST API"""

    # Class-level state shared across requests
    device_state = None

    def log_message(self, format, *args):
        """Override to customize logging"""
        # Log requests to stdout
        pass  # Suppress default logging, we'll print our own

    def do_GET(self):
        """Handle GET requests"""
        parsed_url = urlparse(self.path)
        query_params = parse_qs(parsed_url.query)

        if parsed_url.path == '/options':
            self._handle_get_options(query_params)
        elif parsed_url.path == '/config':
            self._handle_set_config(query_params)
        elif parsed_url.path == '/descriptors':
            self._handle_get_descriptors(query_params)
        else:
            self.send_error(404, "Not Found")

    def _handle_get_options(self, query_params):
        """Handle /options?<param_id> requests"""
        # Extract param_id from query string
        # The query is just the param_id itself, e.g., "eParamID_SWVersion"
        param_id = self.path.split('?')[1] if '?' in self.path else None

        if not param_id:
            self.send_error(400, "Missing parameter ID")
            return

        param_data = self.device_state.get_parameter(param_id)

        if param_data:
            print(f"[GET] {param_id} -> {param_data['text']}")

            # AJA REST API returns array directly, NOT wrapped in object
            response = param_data["options"]

            self._send_json_response(response)
        else:
            print(f"[GET] Unknown parameter: {param_id}")
            self.send_error(404, f"Parameter not found: {param_id}")

    def _handle_set_config(self, query_params):
        """Handle /config?action=set&paramid=<id>&value=<val> requests"""
        # Parse query manually since it's not standard form encoding
        parts = self.path.split('?')[1] if '?' in self.path else ""
        params = {}
        for part in parts.split('&'):
            if '=' in part:
                key, val = part.split('=', 1)
                params[key] = val

        action = params.get('action')
        param_id = params.get('paramid')
        value = params.get('value')

        if action != 'set' or not param_id or value is None:
            self.send_error(400, "Invalid config request")
            return

        # URL decode value
        import urllib.parse
        value = urllib.parse.unquote(value)

        print(f"[SET] {param_id} = {value}")

        success = self.device_state.set_parameter(param_id, value)

        if success:
            # Return success response
            response = {"status": "ok"}
            self._send_json_response(response)
        else:
            self.send_error(400, "Failed to set parameter")

    def _handle_get_descriptors(self, query_params):
        """Handle /descriptors?paramid=* requests (used for caching)"""
        # The client requests all descriptors with paramid=*
        # For simplicity, return empty array - client will fall back to individual queries
        print("[DESCRIPTORS] Requested (returning empty - client will query individually)")
        response = []
        self._send_json_response(response)

    def _send_json_response(self, data):
        """Send JavaScript notation response (mimics real AJA Ki Pro format)

        Real AJA Ki Pro devices return JavaScript notation, NOT valid JSON:
        - Property names are unquoted: {value: "text"} instead of {"value": "text"}
        - Trailing semicolon after arrays: [...];

        This format requires sanitization on the client side.
        """
        # Convert to JSON first, then transform to JavaScript notation
        json_data = json.dumps(data)

        # Remove quotes around property names to match AJA format
        # Transform: {"value": -> {value:
        # This simple replacement works for AJA's predictable format
        js_data = json_data.replace('{"value":', '{value:')
        js_data = js_data.replace(', "text":', ', text:')
        js_data = js_data.replace(', "selected":', ', selected:')

        # Add trailing semicolon for arrays (real hardware format)
        if js_data.startswith('['):
            js_data += ';'

        self.send_response(200)
        self.send_header('Content-Type', 'application/json')
        self.send_header('Content-Length', len(js_data))
        self.end_headers()
        self.wfile.write(js_data.encode('utf-8'))


def main():
    parser = argparse.ArgumentParser(description='Mock AJA Ki Pro Device Simulator')
    parser.add_argument('--port', type=int, default=80, help='Port to listen on (default: 80)')
    parser.add_argument('--ip', type=str, default='0.0.0.0', help='IP address to bind to (default: 0.0.0.0)')
    args = parser.parse_args()

    # Initialize device state
    KiProRequestHandler.device_state = KiProState()

    # Create HTTP server
    server_address = (args.ip, args.port)
    httpd = HTTPServer(server_address, KiProRequestHandler)

    print("=" * 60)
    print(f"Mock Ki Pro Device Simulator")
    print("=" * 60)
    print(f"Listening on: http://{args.ip}:{args.port}")
    print(f"Firmware Version: {KiProRequestHandler.device_state.firmware_version}")
    print()
    print("Waiting for connections from Live Link Hub...")
    print("Press Ctrl+C to stop")
    print("=" * 60)
    print()

    try:
        httpd.serve_forever()
    except KeyboardInterrupt:
        print("\n\nShutting down server...")
        httpd.shutdown()


if __name__ == "__main__":
    main()
