#!/usr/bin/env python3
"""
Test script to verify mock Ki Pro device returns JavaScript notation format.

This test starts the mock server, queries the firmware version endpoint,
and verifies the response format matches real AJA Ki Pro hardware.
"""

import sys
import time
import threading
import http.client
from mock_kipro_device import KiProState


def test_response_format():
    """Test that the mock returns JavaScript notation, not JSON"""

    # Create mock state
    state = KiProState()

    # Get the parameter data
    param_data = state.get_parameter("eParamID_SWVersion")

    if not param_data:
        print("ERROR: Failed to get parameter data")
        return False

    # Manually format as the mock would
    import json
    json_data = json.dumps(param_data["options"])

    # Apply JavaScript notation transformation
    js_data = json_data.replace('{"value":', '{value:')
    js_data = js_data.replace(', "text":', ', text:')
    js_data = js_data.replace(', "selected":', ', selected:')

    if js_data.startswith('['):
        js_data += ';'

    print("Generated response format:")
    print(js_data)
    print()

    # Verify format matches real hardware
    expected_issues = [
        ('Property names should be unquoted', 'value:' in js_data or '{value:' in js_data),
        ('Should have trailing semicolon', js_data.endswith(';')),
        ('Should NOT be valid JSON', '{"value":' not in js_data)
    ]

    all_passed = True
    for description, condition in expected_issues:
        status = "[PASS]" if condition else "[FAIL]"
        print(f"{status} {description}")
        if not condition:
            all_passed = False

    print()

    # Show what the sanitization would transform it to
    print("After sanitization (what C++ will parse):")
    sanitized = js_data.strip()
    if sanitized.endswith(';'):
        sanitized = sanitized[:-1].strip()

    # Simulate regex replacement: {property: -> {"property":
    import re
    pattern = r'([,{]\s*)([a-zA-Z_][a-zA-Z0-9_]*)\s*:'
    sanitized = re.sub(pattern, r'\1"\2":', sanitized)

    print(sanitized)
    print()

    # Verify it's now valid JSON
    try:
        parsed = json.loads(sanitized)
        print("[PASS] Sanitized format is valid JSON")
        print(f"[PASS] Parsed {len(parsed)} item(s)")
        if parsed and len(parsed) > 0:
            item = parsed[0]
            print(f"  - value: {item.get('value')}")
            print(f"  - text: {item.get('text')}")
            print(f"  - selected: {item.get('selected')}")
    except json.JSONDecodeError as e:
        print(f"[FAIL] Sanitized format is NOT valid JSON: {e}")
        all_passed = False

    return all_passed


if __name__ == "__main__":
    print("=" * 60)
    print("Mock Ki Pro JavaScript Notation Format Test")
    print("=" * 60)
    print()

    if test_response_format():
        print()
        print("[PASS] All tests PASSED")
        print()
        print("The mock now correctly mimics real AJA Ki Pro hardware by")
        print("returning JavaScript notation instead of valid JSON.")
        sys.exit(0)
    else:
        print()
        print("[FAIL] Some tests FAILED")
        sys.exit(1)
